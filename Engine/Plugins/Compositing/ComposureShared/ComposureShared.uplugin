{
	"FileVersion": 1,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Composure Shared",
	"Description": "Shared content assets (materials, textures, meshes) migrated from legacy Composure, accessible to both Composure and Composite plugins.",
	"Category": "Compositing",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": false,
	"CanContainContent": true,
	"IsBetaVersion": false,
	"IsExperimentalVersion": false,
	"Installed": false
}
